{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Remote Control Web Interface",
	"Description": "Provides a web interface to control unreal engine via presets, requires nodejs to be installed",
	"Category": "Messaging",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "RemoteControlWebInterface",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Mac",
				"Win64",
				"Linux"
			]
		}
	],
	"Plugins": [
		{
			"Name": "RemoteControl",
			"Enabled": true
		}
	]
}