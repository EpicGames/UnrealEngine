// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCWebInterface.h"

#include "RCWebInterfacePrivate.h"
#include "RCWebInterfaceProcess.h"
#include "RemoteControlSettings.h"
#include "IWebRemoteControlModule.h"
#include "Misc/CommandLine.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Parse.h"

#if WITH_EDITOR
#include "Editor.h"
#include "RCWebInterfaceCustomizations.h"
#else
#include "HAL/IConsoleManager.h"
#endif

#define LOCTEXT_NAMESPACE "FRemoteControlWebInterfaceModule"

static TAutoConsoleVariable<int32> CVarRCWebInterfaceAutoStart(
	TEXT("RCWebInterface.AutoStart"),
	1,
	TEXT("Auto Start Remote Control Web App"));

namespace RCWebInterface
{
	bool IsWebInterfaceEnabled()
	{
		const bool bIsBuildMachine = FPlatformMisc::GetEnvironmentVariable(TEXT("IsBuildMachine")) == TEXT("1");
		
		bool bIsEditor = false;

#if WITH_EDITOR
		bIsEditor = GIsEditor;
#endif

		// By default, remote control web interface is disabled in -game, packaged game, and on build machines.
		return (!bIsBuildMachine && !IsRunningCommandlet() && bIsEditor) || FParse::Param(FCommandLine::Get(), TEXT("RCWebInterfaceEnable"));
	}
}

void FRemoteControlWebInterfaceModule::StartupModule()
{
	WebApp = MakeShared<FRemoteControlWebInterfaceProcess>();

	if (!RCWebInterface::IsWebInterfaceEnabled())
	{
		UE_LOGF(LogRemoteControlWebInterface, Display, "Remote Control Web Interface is disabled by default when running outside the editor. Use the -RCWebInterfaceEnable flag when launching in order to use it.");
		return;
	}

	if (CVarRCWebInterfaceAutoStart.GetValueOnGameThread() == 0)
	{
		UE_LOGF(LogRemoteControlWebInterface, Display, "Remote Control Web Interface did not launch WebApp because CVar RCWebInterface.AutoStart is set to 0");
		return;
	}

	FCoreDelegates::OnFEngineLoopInitComplete.AddLambda([WebAppWeak = WebApp.ToWeakPtr()]()
		{
			const TSharedPtr<FRemoteControlWebInterfaceProcess> WebApp = WebAppWeak.Pin();
			if (WebApp.IsValid() && GetDefault<URemoteControlSettings>()->bAutoStartWebSocketServer)
			{
				WebApp->Start();
			}
		});

	if (IWebRemoteControlModule* WebRemoteControlModule = FModuleManager::GetModulePtr<IWebRemoteControlModule>("WebRemoteControl"))
	{
		WebSocketServerStartedDelegate = WebRemoteControlModule->OnWebSocketServerStarted().AddLambda([this](uint32)
			{
				WebApp->Shutdown();
				WebApp->Start();
			});
	}

#if WITH_EDITOR
	GetMutableDefault<URemoteControlSettings>()->OnSettingChanged().AddRaw(this, &FRemoteControlWebInterfaceModule::OnSettingsModified);

	FCoreDelegates::GetOnPostEngineInit().AddLambda([this]()
		{
			Customizations = MakePimpl<FRCWebInterfaceCustomizations>(WebApp);
		});
#endif
}

void FRemoteControlWebInterfaceModule::ShutdownModule()
{
	if (WebApp.IsValid())
	{
		WebApp->Shutdown();
	}

	WebApp.Reset();
	Customizations.Reset();

	FCoreDelegates::OnFEngineLoopInitComplete.RemoveAll(this);

#if WITH_EDITOR
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
#endif
	
	if (IWebRemoteControlModule* WebRemoteControlModule = FModuleManager::GetModulePtr<IWebRemoteControlModule>("WebRemoteControl"))
	{
		WebRemoteControlModule->OnWebSocketServerStarted().Remove(WebSocketServerStartedDelegate);
		WebSocketServerStartedDelegate.Reset();
	}
}

bool FRemoteControlWebInterfaceModule::Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar)
{
	if (!FParse::Command(&Cmd, TEXT("RCWebInterface")))
	{
		return false;
	}

	FRemoteControlWebInterfaceModule& RCWebInf = FRemoteControlWebInterfaceModule::Get();
	const FRemoteControlWebInterfaceProcess::EStatus WebAppStatus = RCWebInf.WebApp->GetStatus();

	const bool bIsCurrentlyRunning = (
		WebAppStatus == FRemoteControlWebInterfaceProcess::EStatus::Running ||
		WebAppStatus == FRemoteControlWebInterfaceProcess::EStatus::Launching
	);

	if (FParse::Command(&Cmd, TEXT("Start")))
	{
		if (!bIsCurrentlyRunning) // Ignore if already running
		{
			Ar.Log(TEXT("RCWebInterface: Starting WebApp"));

			RCWebInf.WebApp->Start();
		}
		else
		{
			Ar.Log(TEXT("RCWebInterface: WebApp was already running"));
		}
	}
	else if (FParse::Command(&Cmd, TEXT("Stop")))
	{
		Ar.Log(TEXT("RCWebInterface: Stopping WebApp"));

		RCWebInf.WebApp->Shutdown();
	}
	else if (FParse::Command(&Cmd, TEXT("Restart")))
	{
		Ar.Log(TEXT("RCWebInterface: Restarting WebApp"));

		RCWebInf.WebApp->Shutdown();
		RCWebInf.WebApp->Start();
	}

	return true;
}

void FRemoteControlWebInterfaceModule::OnSettingsModified(UObject* Settings, FPropertyChangedEvent& PropertyChangedEvent)
{
	FName PropertyName = PropertyChangedEvent.GetPropertyName();

	if (PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
	{
		if (PropertyName == GET_MEMBER_NAME_CHECKED(URemoteControlSettings, RemoteControlWebInterfacePort)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(URemoteControlSettings, bForceWebAppBuildAtStartup)
			|| PropertyName == GET_MEMBER_NAME_CHECKED(URemoteControlSettings, RemoteControlWebInterfaceBindAddress))
		{
			WebApp->Shutdown();
			WebApp->Start();
		}
	}


	if (PropertyName == GET_MEMBER_NAME_CHECKED(URemoteControlSettings, bWebAppLogRequestDuration))
	{
		const URemoteControlSettings* RCSettings = CastChecked<URemoteControlSettings>(Settings);
		WebApp->SetExternalLoggerEnabled(RCSettings->bWebAppLogRequestDuration);
	}
}

#undef LOCTEXT_NAMESPACE

DEFINE_LOG_CATEGORY(LogRemoteControlWebInterface);

IMPLEMENT_MODULE(FRemoteControlWebInterfaceModule, RemoteControlWebInterface)