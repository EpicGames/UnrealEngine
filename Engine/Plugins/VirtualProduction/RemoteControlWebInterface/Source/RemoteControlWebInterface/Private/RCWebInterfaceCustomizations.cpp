// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCWebInterfaceCustomizations.h"

#if WITH_EDITOR
#include "Controller/RCStatelessEventController.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "EditorFontGlyphs.h"
#include "HAL/PlatformProcess.h"
#include "IRemoteControlModule.h"
#include "IRemoteControlUIModule.h"
#include "ISettingsModule.h"
#include "Input/Reply.h"
#include "RCVirtualProperty.h"
#include "RCWebInterfaceProcess.h"
#include "RemoteControlEntity.h"
#include "RemoteControlPreset.h"
#include "RemoteControlSettings.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "ScopedTransaction.h"
#include "Styling/AppStyle.h"
#include "UObject/TextProperty.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "FRCWebInterfaceCustomizations"

namespace RCWebInterface
{
	static const FName MetadataKeyName_Widget = FName("Widget");
	static const FText MetadataKey_Widget = LOCTEXT("Widget", "Widget");
	static const FName MetadataKeyName_Description = FName("Description");
	static const FText MetadataKey_Description = LOCTEXT("Description", "Description");

	TArray<FString> GetSupportedWidgets(FProperty* Property)
	{
		if (!Property)
		{
			return TArray<FString>();
		}

		TArray<FString> Widgets;
		if (Property->IsA<FNumericProperty>() && !Property->IsA<FByteProperty>())
		{
			Widgets = { TEXT("Slider"), TEXT("Dial") };
		}
		else if (Property->IsA<FBoolProperty>())
		{
			Widgets = { TEXT("Toggle") };
		}
		else if (Property->IsA<FTextProperty>() || Property->IsA<FNameProperty>() || Property->IsA<FStrProperty>())
		{
			Widgets = { TEXT("Text") };
		}
		else if (Property->IsA<FByteProperty>())
		{
			FByteProperty* ByteProperty = CastField<FByteProperty>(Property);
			if (ByteProperty->IsEnum())
			{
				Widgets = { TEXT("Dropdown") };
			}
			else
			{
				Widgets = { TEXT("Slider"), TEXT("Dial") };
			}
		}
		else if (Property->IsA<FEnumProperty>())
		{
			Widgets = { TEXT("Dropdown") };
		}
		else if (Property->IsA<FStructProperty>())
		{
			FStructProperty* StructProperty = CastField<FStructProperty>(Property);
			if (UStruct* Struct = StructProperty->Struct)
			{
				if (Struct->IsChildOf(TBaseStructure<FVector>::Get()))
				{
					Widgets = { TEXT("Vector"), TEXT("Joystick"), TEXT("Sliders"), TEXT("Dials") };
				}
				else if (Struct->IsChildOf(TBaseStructure<FVector2D>::Get()))
				{
					Widgets = { TEXT("Vector"), TEXT("Joystick"), TEXT("Sliders"), TEXT("Dials") };
				}
				else if (Struct->IsChildOf(TBaseStructure<FRotator>::Get()))
				{
					Widgets = { TEXT("Vector"), TEXT("Sliders"), TEXT("Dials") };
				}
				else if (Struct->IsChildOf(TBaseStructure<FVector4>::Get()) || Struct->IsChildOf(TBaseStructure<FColor>::Get()) || Struct->IsChildOf(TBaseStructure<FLinearColor>::Get()))
				{
					Widgets = { TEXT("Color Picker"), TEXT("Mini Color Picker") };
				}
			}
		}
		else if (Property->IsA<FObjectPropertyBase>())
		{
			Widgets = { TEXT("Asset") };
		}

		return Widgets;
	}

	FString GetDefaultWidget(FProperty* Property)
	{
		FString Widget;
		TArray<FString> SupportedWidgets = GetSupportedWidgets(Property);
		if (SupportedWidgets.Num())
		{
			Widget = SupportedWidgets[0];
		}
		
		return Widget;
	}
	
	FString GetDefaultWidget(URemoteControlPreset* Preset, const FGuid& Id)
	{
		FString Widget;
		if (const UScriptStruct* EntityStruct = Preset->GetExposedEntityType(Id))
		{
			if (EntityStruct->IsChildOf(FRemoteControlProperty::StaticStruct()))
			{
				if (TSharedPtr<FRemoteControlProperty> Property = Preset->GetExposedEntity<FRemoteControlProperty>(Id).Pin())
				{
					Widget = RCWebInterface::GetDefaultWidget(Property->GetProperty());
				}
			}
			else if (EntityStruct->IsChildOf(FRemoteControlFunction::StaticStruct()))
			{
				Widget = TEXT("Button");
			}
		}

		return Widget;
	}

	TArray<FString> GetSupportedWidgets(URemoteControlPreset* Preset, const FGuid& Id)
	{
		TArray<FString> Widgets;
		if (const UScriptStruct* EntityStruct = Preset->GetExposedEntityType(Id))
		{
			if (EntityStruct->IsChildOf(FRemoteControlProperty::StaticStruct()))
			{
				if (TSharedPtr<FRemoteControlProperty> Property = Preset->GetExposedEntity<FRemoteControlProperty>(Id).Pin())
				{
					Widgets = GetSupportedWidgets(Property->GetProperty());
				}
			}
			else if (EntityStruct->IsChildOf(FRemoteControlFunction::StaticStruct()))
			{
				Widgets = { TEXT("Button") };
			}
		}

		return Widgets;
	}
}

FRCWebInterfaceCustomizations::FRCWebInterfaceCustomizations(TSharedPtr<FRemoteControlWebInterfaceProcess> Process)
{
	WebApp = MoveTemp(Process);
	RegisterPanelExtension();
	RegisterExposedEntityCallback();
	RegisterPanelMetadataCustomization();
}

FRCWebInterfaceCustomizations::~FRCWebInterfaceCustomizations()
{
	UnregisterPanelMetadataCustomization();
	UnregisterExposedEntityCallback();
	UnregisterPanelExtension();
}

void FRCWebInterfaceCustomizations::RegisterPanelExtension()
{
	if (IRemoteControlUIModule* RemoteControlUI = FModuleManager::GetModulePtr<IRemoteControlUIModule>("RemoteControlUI"))
	{
		RemoteControlUI->GetExtensionGenerators().AddRaw(this, &FRCWebInterfaceCustomizations::GeneratePanelExtensions);
	}
}

void FRCWebInterfaceCustomizations::UnregisterPanelExtension()
{
	if (IRemoteControlUIModule* RemoteControlUI = FModuleManager::GetModulePtr<IRemoteControlUIModule>("RemoteControlUI"))
	{
		RemoteControlUI->GetExtensionGenerators().RemoveAll(this);
	}
}

void FRCWebInterfaceCustomizations::GeneratePanelExtensions(TArray<TSharedRef<SWidget>>& OutExtensions)
{
	auto GetDetailWidgetIndex = [this]()
	{
		if (WebApp->GetStatus() == FRemoteControlWebInterfaceProcess::EStatus::Launching)
		{
			return 0;	
		}
		else
		{
			return 1;
		}
	};

	TSharedPtr<SWidget> Throbber;

	OutExtensions.Add(
		SNew(SButton)
		.ButtonStyle(FAppStyle::Get(), "FlatButton")
		.ToolTipText(LOCTEXT("AppPassphraseDisabled", "Warning: Passphrase is disabled!"))
		.Visibility_Lambda([]()
		{
			const URemoteControlSettings* RCSettings = GetDefault<URemoteControlSettings>();
			bool bShouldWarn = RCSettings->bShowPassphraseDisabledWarning && !RCSettings->bEnforcePassphraseForRemoteClients;
			return bShouldWarn ? EVisibility::Visible : EVisibility::Collapsed;
		})
		.OnClicked_Lambda([]()
		{
			URemoteControlSettings* RCSettings = GetMutableDefault<URemoteControlSettings>();
			RCSettings->bShowPassphraseDisabledWarning = false;
			RCSettings->SaveConfig();
			
			return FReply::Handled();
		})
		[
			SNew(STextBlock)
			.TextStyle(FAppStyle::Get(), "NormalText.Important")
			.Font(FAppStyle::Get().GetFontStyle("FontAwesome.10"))
			.Text(FEditorFontGlyphs::Exclamation_Triangle)
		]
	);
	
	OutExtensions.Add(
		SNew(SWidgetSwitcher)
		.WidgetIndex_Lambda(GetDetailWidgetIndex)
		+ SWidgetSwitcher::Slot()
		[
			SAssignNew(Throbber, SCircularThrobber)
			.Radius(10.f)
		]
		+ SWidgetSwitcher::Slot()
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "FlatButton")
			.ToolTipText_Lambda([this]()
				{
					switch (WebApp->GetStatus())
					{
					case FRemoteControlWebInterfaceProcess::EStatus::Error: return LOCTEXT("AppErrorTooltip", "An error occurred when launching the web app.\r\nClick to change the Web App's settings.");
					case FRemoteControlWebInterfaceProcess::EStatus::Running: return LOCTEXT("AppRunningTooltip", "Open the web app in a web browser.");
					case FRemoteControlWebInterfaceProcess::EStatus::Stopped: return LOCTEXT("AppStoppedTooltip", "The web app is not currently running.\r\nClick to change the Web App's settings.");
					default: return FText();
					}
				})
			.OnClicked_Raw(this, &FRCWebInterfaceCustomizations::OpenWebApp)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0.f, 2.f, 4.f, 2.f)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "NormalText.Important")
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.14"))
					.Text_Lambda([this]()
						{
							switch (WebApp->GetStatus())
							{
							case FRemoteControlWebInterfaceProcess::EStatus::Error: return FEditorFontGlyphs::Exclamation_Triangle;
							case FRemoteControlWebInterfaceProcess::EStatus::Running: return FEditorFontGlyphs::External_Link;
							case FRemoteControlWebInterfaceProcess::EStatus::Stopped: return FEditorFontGlyphs::Exclamation_Triangle;
							default: return FEditorFontGlyphs::Exclamation_Triangle;
							}
						})
				]

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Center)
				.FillWidth(1.f)
				.Padding(4.f, 2.f, 0.f, 2.f)
				[
					SNew(STextBlock)
					.Text_Lambda([this]()
					{
						switch (WebApp->GetStatus())
						{
							case FRemoteControlWebInterfaceProcess::EStatus::Error: return LOCTEXT("ErrorText", "Error");
							case FRemoteControlWebInterfaceProcess::EStatus::Running: return LOCTEXT("RunningText", "Web App");
							case FRemoteControlWebInterfaceProcess::EStatus::Stopped: return LOCTEXT("StoppedText", "Stopped");
							default: return LOCTEXT("UnknownErrorText", "Unknown Error");
						}
					})
				]
			]
		]
	);

	Throbber->SetToolTipText(LOCTEXT("AppLaunchingTooltip", "The web app is in the process of launching."));
}

FReply FRCWebInterfaceCustomizations::OpenWebApp() const
{
	if (WebApp->GetStatus() == FRemoteControlWebInterfaceProcess::EStatus::Running)
	{
		const uint32 Port = GetDefault<URemoteControlSettings>()->RemoteControlWebInterfacePort;
		FString ActivePreset;
		if (URemoteControlPreset* Preset = IRemoteControlUIModule::Get().GetActivePreset())
		{
			ActivePreset = Preset->GetPresetId().ToString();	
		}
		else
		{
			ensureMsgf(false, TEXT("Active preset was invalid when the launch web app button was pressed."));
		}
		
		const FString Address = FString::Printf(TEXT("http://127.0.0.1:%d/?preset=%s"), Port, *ActivePreset);

		FPlatformProcess::LaunchURL(*Address, nullptr, nullptr);
	}
	else
	{
		FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer("Project", "Plugins", "RemoteControlWebInterface");
	}

	return FReply::Handled();
}

void FRCWebInterfaceCustomizations::RegisterExposedEntityCallback() const
{
	IRemoteControlModule& RemoteControlModule = FModuleManager::LoadModuleChecked<IRemoteControlModule>("RemoteControl");
	RemoteControlModule.RegisterDefaultEntityMetadata(RCWebInterface::MetadataKeyName_Widget, FEntityMetadataInitializer::CreateRaw(this, &FRCWebInterfaceCustomizations::OnInitializeWidgetMetadata));
	RemoteControlModule.RegisterDefaultEntityMetadata(RCWebInterface::MetadataKeyName_Description, FEntityMetadataInitializer::CreateLambda([](URemoteControlPreset* Preset, const FGuid& EntityId) { return FString(""); }));
}

void FRCWebInterfaceCustomizations::UnregisterExposedEntityCallback() const
{
	if (IRemoteControlModule* RemoteControlModule = FModuleManager::GetModulePtr<IRemoteControlModule>("RemoteControl"))
	{
		RemoteControlModule->UnregisterDefaultEntityMetadata(RCWebInterface::MetadataKeyName_Widget);
		RemoteControlModule->UnregisterDefaultEntityMetadata(RCWebInterface::MetadataKeyName_Description);
	}
}

void FRCWebInterfaceCustomizations::RegisterPanelMetadataCustomization()
{
	if (IRemoteControlUIModule* UIModule = FModuleManager::GetModulePtr<IRemoteControlUIModule>("RemoteControlUI"))
	{
		UIModule->RegisterMetadataCustomization(RCWebInterface::MetadataKeyName_Widget, FOnCustomizeMetadataEntry::CreateRaw(this, &FRCWebInterfaceCustomizations::CustomizeWidgetTypeMetadata));
		UIModule->RegisterMetadataCustomization(RCWebInterface::MetadataKeyName_Description, FOnCustomizeMetadataEntry::CreateRaw(this, &FRCWebInterfaceCustomizations::CustomizeWidgetDescriptionMetadata));
	}
}

void FRCWebInterfaceCustomizations::UnregisterPanelMetadataCustomization()
{
	if (IRemoteControlUIModule* UIModule = FModuleManager::GetModulePtr<IRemoteControlUIModule>("RemoteControlUI"))
	{
		UIModule->UnregisterMetadataCustomization(RCWebInterface::MetadataKeyName_Widget);
		UIModule->UnregisterMetadataCustomization(RCWebInterface::MetadataKeyName_Description);
	}
}

void FRCWebInterfaceCustomizations::CustomizeWidgetTypeMetadata(URemoteControlPreset* Preset, const FGuid& DisplayedEntityId, IDetailLayoutBuilder& LayoutBuilder, IDetailCategoryBuilder& CategoryBuilder)
{
	const FSlateFontInfo FontInfo = LayoutBuilder.GetDetailFont();
	check(Preset);

	TWeakPtr<FRemoteControlEntity> WeakEntity = Preset->GetExposedEntity<FRemoteControlEntity>(DisplayedEntityId);
	EntityBeingDisplayed = WeakEntity;
	
	FString InitialWidgetTypeContent = EntityBeingDisplayed.IsValid() ? EntityBeingDisplayed.Pin()->GetMetadata().FindChecked(RCWebInterface::MetadataKeyName_Widget) : TEXT("");

	WidgetTypes.Reset();
	Algo::Transform(RCWebInterface::GetSupportedWidgets(Preset, DisplayedEntityId), WidgetTypes, [](const FString& InWidget) { return MakeShared<FString>(InWidget); });

	if (WidgetTypes.Num())
	{
		FDetailWidgetRow& WidgetTypeRow = CategoryBuilder.AddCustomRow( RCWebInterface::MetadataKey_Widget)
		.NameContent()
		[
			SNew(STextBlock)
			.Font(FontInfo)
			.Text(RCWebInterface::MetadataKey_Widget)
		]
		.ValueContent()
		[
    		SAssignNew(SearchableBox, SSearchableComboBox)
    		.OnSelectionChanged_Raw(this, &FRCWebInterfaceCustomizations::OnWidgetSelectionChanged)
    		.OnGenerateWidget_Lambda([](const TSharedPtr<FString>& InItem)
    		{
    			return SNew(STextBlock)
    				.Text(InItem ? FText::FromString(*InItem) : FText::GetEmpty());
    		})
    		.OptionsSource(&WidgetTypes)
    		.InitiallySelectedItem(MakeShared<FString>(MoveTemp(InitialWidgetTypeContent)))
    		.Content()
    		[
				SNew(STextBlock)
				.Text_Lambda([WeakEntity]()
				{
					FText EntryText = FText::GetEmpty();
					if (TSharedPtr<FRemoteControlEntity> Entity = WeakEntity.Pin())
					{
						if (const FString* WidgetEntry = Entity->GetMetadata().Find(RCWebInterface::MetadataKeyName_Widget))
						{
							EntryText = FText::FromString(*WidgetEntry);
						}
					}
					return EntryText;
				})
    		]
		];
	}
	
}

void FRCWebInterfaceCustomizations::CustomizeWidgetDescriptionMetadata(URemoteControlPreset* Preset, const FGuid& DisplayedEntityId, IDetailLayoutBuilder& LayoutBuilder, IDetailCategoryBuilder& CategoryBuilder)
{
	const FSlateFontInfo FontInfo = LayoutBuilder.GetDetailFont();
	check(Preset);

	TWeakPtr<FRemoteControlEntity> WeakEntity = Preset->GetExposedEntity<FRemoteControlEntity>(DisplayedEntityId);
	EntityBeingDisplayed = WeakEntity;
	
	FDetailWidgetRow& DescriptionRow = CategoryBuilder.AddCustomRow( RCWebInterface::MetadataKey_Description)
	.NameContent()
	[
		SNew(STextBlock)
		.Font(FontInfo)
		.Text(RCWebInterface::MetadataKey_Description)
	]
	.ValueContent()
	[
    	SAssignNew(DescriptionBox, SEditableTextBox)
    	.OnTextCommitted_Raw(this, &FRCWebInterfaceCustomizations::OnWidgetDescriptionChanged)
		.Text_Lambda([WeakEntity]()
			{
				FText EntryText = FText::GetEmpty();
				if (TSharedPtr<FRemoteControlEntity> Entity = WeakEntity.Pin())
				{
					if (const FString* WidgetEntry = Entity->GetMetadata().Find(RCWebInterface::MetadataKeyName_Description))
					{
						EntryText = FText::FromString(*WidgetEntry);
					}
				}
				return EntryText;
			})
	];
}

void FRCWebInterfaceCustomizations::OnWidgetSelectionChanged(TSharedPtr<FString> InItem, ESelectInfo::Type) const
{
	if (InItem)
	{
		if (TSharedPtr<FRemoteControlEntity> Entity = EntityBeingDisplayed.Pin())
		{
			if (URemoteControlPreset* Owner= Entity->GetOwner())
			{
				FScopedTransaction Transaction(LOCTEXT("ModifyEntityMetadata", "Modify exposed entity metadata"));
				Owner->Modify();
				Entity->SetMetadataValue(RCWebInterface::MetadataKeyName_Widget, *InItem);
			}
		}
	}
}

void FRCWebInterfaceCustomizations::OnWidgetDescriptionChanged(const FText& InDescription, ETextCommit::Type) const
{
	if (TSharedPtr<FRemoteControlEntity> Entity = EntityBeingDisplayed.Pin())
	{
		if (URemoteControlPreset* Owner = Entity->GetOwner())
		{
			FScopedTransaction Transaction(LOCTEXT("ModifyEntityMetadata", "Modify exposed entity metadata"));
			Owner->Modify();
			Entity->SetMetadataValue(RCWebInterface::MetadataKeyName_Description, InDescription.ToString());
		}
	}
}

FString FRCWebInterfaceCustomizations::OnInitializeWidgetMetadata(URemoteControlPreset* Preset, const FGuid& EntityId) const
{
	check(Preset);
	FString Metadata;
	
	if (const UScriptStruct* EntityStruct = Preset->GetExposedEntityType(EntityId))
	{
		if (EntityStruct->IsChildOf(FRemoteControlProperty::StaticStruct()))
		{
			if (TSharedPtr<FRemoteControlProperty> Property = Preset->GetExposedEntity<FRemoteControlProperty>(EntityId).Pin())
			{
				Metadata = RCWebInterface::GetDefaultWidget(Property->GetProperty());
			}
		}
		else if (EntityStruct->IsChildOf(FRemoteControlFunction::StaticStruct()))
		{
			Metadata = TEXT("Button");
		}
	}
	else if (const URCVirtualPropertyBase* Controller = Preset->GetController(EntityId))
	{
		if (FRCStatelessEventController::IsStatelessEventController(Controller))
		{
			Metadata = TEXT("Button");
		}
		else
		{
			Metadata = RCWebInterface::GetDefaultWidget(const_cast<FProperty*>(Controller->GetProperty()));
		}
	}
	
	return Metadata;
}

#undef LOCTEXT_NAMESPACE /* FRCWebInterfaceCustomizations */
#endif /*WITH_EDITOR*/
