// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "Types/SlateEnums.h"

class FRemoteControlWebInterfaceProcess;
struct FRemoteControlEntity;
struct FGuid;
class FReply;
class IDetailCategoryBuilder;
class IDetailLayoutBuilder;
class SSearchableComboBox;
class SEditableTextBox;
class SWidget;
class URemoteControlPreset;

/**
 * Handles registering customizations acting on the remote control panel.
 */
class FRCWebInterfaceCustomizations
{
public:
	FRCWebInterfaceCustomizations(TSharedPtr<FRemoteControlWebInterfaceProcess> Process);
	~FRCWebInterfaceCustomizations();
	
private:
	//~ Handle registering a widget extension with the remote control panel to show the web app's status.
	void RegisterPanelExtension();
	void UnregisterPanelExtension();

	/** Handles adding the widget extension to the remote control panel. */
	void GeneratePanelExtensions(TArray<TSharedRef<SWidget>>& OutExtensions);

	/** Handles launching the web app through a web browser. */
	class FReply OpenWebApp() const;

	//~ Register RC Interface metadata with the RC Module. 
	void RegisterExposedEntityCallback() const;
	void UnregisterExposedEntityCallback() const;

	//~ Register/Unregister metadata customizations relevant to the web interface.
	void RegisterPanelMetadataCustomization();
	void UnregisterPanelMetadataCustomization();

	/** Handles customizing the widget type metadata entry. */
	void CustomizeWidgetTypeMetadata(URemoteControlPreset* Preset, const FGuid& DisplayedEntityId, IDetailLayoutBuilder& LayoutBuilder, IDetailCategoryBuilder& CategoryBuilder);

	/** Handles customizing the widget description metadata entry. */
	void CustomizeWidgetDescriptionMetadata(URemoteControlPreset* Preset, const FGuid& DisplayedEntityId, IDetailLayoutBuilder& LayoutBuilder, IDetailCategoryBuilder& CategoryBuilder);

	/** Handles selecting a different widget representation for an entity. */
	void OnWidgetSelectionChanged(TSharedPtr<FString> InItem, ESelectInfo::Type) const;

	/** Handles widget description change. */
	void OnWidgetDescriptionChanged(const FText& InDescription, ETextCommit::Type) const;

	/** Handles initializing the widget metadata for entities. */
	FString OnInitializeWidgetMetadata(URemoteControlPreset* Preset, const FGuid& EntityId) const;
	
private:
	/** The actual process that runs the middleman server. */
	TSharedPtr<class FRemoteControlWebInterfaceProcess> WebApp;
	
	/** List of widgets displayed in the metadata widget combo box. */
	TArray<TSharedPtr<FString>> WidgetTypes;

	/** Holds the entity currently being displayed in the RC Panel. */
	TWeakPtr<FRemoteControlEntity> EntityBeingDisplayed;

	/** Holds the searchable box used for picking a widget for a given entity. */
	TSharedPtr<SSearchableComboBox> SearchableBox;

	/** Holds the textbox used for specifying the description for a given entity. */
	TSharedPtr<SEditableTextBox> DescriptionBox;
};

#else
//~ Stub class for non-editor builds.
class FRCWebInterfaceCustomizations
{
};

#endif
