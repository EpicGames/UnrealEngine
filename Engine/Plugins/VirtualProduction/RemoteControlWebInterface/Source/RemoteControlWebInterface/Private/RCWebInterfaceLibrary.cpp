// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCWebInterfaceLibrary.h"

#include "Engine/World.h"
#include "IRemoteControlModule.h"
#include "Kismet/GameplayStatics.h"
#include "RCWebInterfacePrivate.h"
#include "RemoteControlBinding.h"
#include "RemoteControlPreset.h"
#include "StructSerializer.h"
#include "IStructSerializerBackend.h"
#include "Backends/JsonStructSerializerBackend.h"


#define LOCTEXT_NAMESPACE "RemoteControlWebInterface"


TMap<FString, AActor*> URCWebInterfaceBlueprintLibrary::FindMatchingActorsToRebind(const FString& PresetId, const TArray<FString>& PropertyIds)
{
	TSet<AActor*> MatchesIntersection;
	if (URemoteControlPreset* RCPreset = URCWebInterfaceBlueprintLibrary::GetPreset(PresetId))
	{
		for (const FString& PropertyId : PropertyIds)
		{
			const FGuid PropertyGuid(PropertyId);
			TWeakPtr<FRemoteControlField> ExposedEntity = RCPreset->GetExposedEntity<FRemoteControlField>(PropertyGuid);

			TSet<AActor*> MatchingActorsForProperty;
			if (TSharedPtr<FRemoteControlField> RemoteControlField = ExposedEntity.Pin())
			{
				TArray<UObject*> BoundObjects = RemoteControlField->GetBoundObjects();
				if (BoundObjects.Num() >= 1)
				{
					UObject* OwnerObject = BoundObjects[0];
					if (AActor* Actor = Cast<AActor>(OwnerObject))
					{
						TArray<AActor*> MatchingActors;
						UGameplayStatics::GetAllActorsOfClass(Actor->GetWorld(), Actor->GetClass(), MatchingActors);
						MatchingActorsForProperty.Append(MatchingActors);
					}
					else if (UActorComponent* ActorComponent = Cast<UActorComponent>(OwnerObject))
					{
						AActor* OwnerActor = ActorComponent->GetOwner();

						TArray<AActor*> MatchingActors;
						UGameplayStatics::GetAllActorsOfClass(OwnerActor->GetWorld(), OwnerActor->GetClass(), MatchingActors);
						for (AActor* MatchingActor : MatchingActors)
						{
							if (UActorComponent* MatchingComponent = MatchingActor->GetComponentByClass(ActorComponent->GetClass()))
							{
								MatchingActorsForProperty.Add(MatchingActor);
							}
						}
					}
				}
				else if (UClass* SupportedBindingClass = RemoteControlField->GetSupportedBindingClass())
				{
					constexpr bool bAllowPIE = false;
					UWorld* World = RCPreset->GetWorld(bAllowPIE);
					if (SupportedBindingClass->IsChildOf(AActor::StaticClass()))
					{
						TArray<AActor*> MatchingActors;
						UGameplayStatics::GetAllActorsOfClass(World, SupportedBindingClass, MatchingActors);
						MatchingActorsForProperty.Append(MatchingActors);
					}
					else if (SupportedBindingClass->IsChildOf(UActorComponent::StaticClass()))
					{
						TArray<AActor*> MatchingActors;
						UGameplayStatics::GetAllActorsOfClass(World, AActor::StaticClass(), MatchingActors);
						for (AActor* MatchingActor : MatchingActors)
						{
							if (UActorComponent* MatchingComponent = MatchingActor->GetComponentByClass(SupportedBindingClass))
							{
								MatchingActorsForProperty.Add(MatchingActor);
							}
						}
					}
				}
			}

			if (MatchesIntersection.Num() == 0)
			{
				MatchesIntersection = MatchingActorsForProperty;
			}
			else
			{
				for (auto Iterator = MatchesIntersection.CreateIterator(); Iterator; ++Iterator)
				{
					if (!MatchingActorsForProperty.Contains(*Iterator))
					{
						Iterator.RemoveCurrent();
					}
				}
			}
		}
	}

	TMap<FString, AActor*> Matches;
	Matches.Reserve(MatchesIntersection.Num());
	for (AActor* MatchingActor : MatchesIntersection)
	{
		Matches.Add(GetActorNameOrLabel(MatchingActor), MatchingActor);
	}

	return Matches;
}

FString URCWebInterfaceBlueprintLibrary::GetOwnerActorLabel(const FString& PresetId, const TArray<FString>& PropertyIds)
{
	TSet<FString> Actors;
	if (URemoteControlPreset* RCPreset = URCWebInterfaceBlueprintLibrary::GetPreset(PresetId))
	{
		for (const FString& PropertyId : PropertyIds)
		{
			const FGuid PropertyGuid(PropertyId);
			TWeakPtr<FRemoteControlField> ExposedEntity = RCPreset->GetExposedEntity<FRemoteControlField>(PropertyGuid);
			if (TSharedPtr<FRemoteControlField> RemoteControlField = ExposedEntity.Pin())
			{
				TArray<UObject*> BoundObjects = RemoteControlField->GetBoundObjects();
				if (BoundObjects.Num() >= 1)
				{
					UObject* OwnerObject = BoundObjects[0];
					if (const AActor* Actor = Cast<AActor>(OwnerObject))
					{
						Actors.Add(GetActorNameOrLabel(Actor));
					}
					else if (const UActorComponent* ActorComponent = Cast<UActorComponent>(OwnerObject))
					{
						AActor* OwnerActor = ActorComponent->GetOwner();
						Actors.Add(GetActorNameOrLabel(OwnerActor));
					}
				}
			}
		}
	}

	if (Actors.Num() == 1)
	{
		return Actors.Array()[0];
	}

	if (Actors.Num() > 1)
	{
		UE_LOGF(LogRemoteControlWebInterface, Warning, "Propreties have different owners");
	}

	return FString();
}

void URCWebInterfaceBlueprintLibrary::RebindProperties(const FString& PresetId, const TArray<FString>& PropertyIds, AActor* NewOwner)
{
	if (URemoteControlPreset* RCPreset = URCWebInterfaceBlueprintLibrary::GetPreset(PresetId))
	{
		for (const FString& PropertyId : PropertyIds)
		{
			const FGuid PropertyGuid(PropertyId);
			TWeakPtr<FRemoteControlField> ExposedEntity = RCPreset->GetExposedEntity<FRemoteControlField>(PropertyGuid);
			if (TSharedPtr<FRemoteControlField> RemoteControlField = ExposedEntity.Pin())
			{
				RemoteControlField->BindObject(NewOwner);
			}
		}
	}
}

URemoteControlPreset* URCWebInterfaceBlueprintLibrary::GetPreset(const FString& PresetId)
{
	FGuid Id;
	if (FGuid::ParseExact(PresetId, EGuidFormats::Digits, Id))
	{
		if (URemoteControlPreset* ResolvedPreset = IRemoteControlModule::Get().ResolvePreset(Id))
		{
			return ResolvedPreset;
		}
	}

	return IRemoteControlModule::Get().ResolvePreset(*PresetId);
}

FString URCWebInterfaceBlueprintLibrary::GetActorNameOrLabel(const AActor* Actor)
{
#if WITH_EDITOR
	return Actor->GetActorLabel();
#else
	return Actor->GetName();
#endif
}

TMap<AActor*, FString> URCWebInterfaceBlueprintLibrary::FindAllActorsOfClass(UClass* Class)
{
	UWorld* World = URemoteControlPreset::GetWorld(nullptr, false);
	if (!World || !Class)
	{
		return {};
	}

	TArray<AActor*> FoundActors;
	UGameplayStatics::GetAllActorsOfClass(World, Class, FoundActors);

	TMap<AActor*, FString> Results;
	for (AActor* Actor : FoundActors)
	{
		Results.Add(Actor, GetActorNameOrLabel(Actor));
	}

	return Results;
}

AActor* URCWebInterfaceBlueprintLibrary::SpawnActor(UClass* Class)
{
	UWorld* World = URemoteControlPreset::GetWorld(nullptr, false);
	if (!World || !Class)
	{
		return nullptr;
	}

	return World->SpawnActor<AActor>(Class, FActorSpawnParameters());
}

TMap<AActor*, FString> URCWebInterfaceBlueprintLibrary::GetValuesOfActorsByClass(UClass* Class)
{
	UWorld* World = URemoteControlPreset::GetWorld(nullptr, false);
	if (!World || !Class)
	{
		return {};
	}

	TArray<AActor*> FoundActors;
	UGameplayStatics::GetAllActorsOfClass(World, Class, FoundActors);

	TMap<AActor*, FString> Values;
	Values.Reserve(FoundActors.Num());

	for (AActor* Actor : FoundActors)
	{
		TArray<uint8> WorkingBuffer;
		FMemoryWriter Writer(WorkingBuffer);
		FJsonStructSerializerBackend SerializerBackend(Writer, EStructSerializerBackendFlags::Default);

		FStructSerializer::Serialize(Actor, *Class, SerializerBackend, FStructSerializerPolicies());

		// Add 16bit \0
		WorkingBuffer.AddDefaulted(2);
		Values.Add(Actor, TCHAR_TO_UTF8(WorkingBuffer.GetData()));
	}

	return Values;
}


#undef LOCTEXT_NAMESPACE