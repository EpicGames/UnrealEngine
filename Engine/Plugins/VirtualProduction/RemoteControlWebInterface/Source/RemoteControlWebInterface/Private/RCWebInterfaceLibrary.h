// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "RCWebInterfaceLibrary.generated.h"

UCLASS()
class URCWebInterfaceBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	/**
	 * Get a list of compatible actors to rebind the remote control preset properties
	 * returned value is a map, key is the actor label
	 */
	UFUNCTION(BlueprintCallable, Category = "RemoteControlWebInterface")
	static TMap<FString, AActor*> FindMatchingActorsToRebind(const FString& PresetId, const TArray<FString>& PropertyIds);

	/**
	 * Get the label of the owner actor of the remote control preset properties
	 * If the properties has different owners, an empty string will be returned
	 */
	UFUNCTION(BlueprintCallable, Category = "RemoteControlWebInterface")
	static FString GetOwnerActorLabel(const FString& PresetId, const TArray<FString>& PropertyIds);

	/**
	 * Rebind the remote control preset properties to a new owner
	 */
	UFUNCTION(BlueprintCallable, Category = "RemoteControlWebInterface")
	static void RebindProperties(const FString& PresetId, const TArray<FString>& PropertyIds, AActor* NewOwner);

	/**
	 * Shortcut function to find all actors of a class
	 */
	UFUNCTION(BlueprintCallable, Category = "RemoteControlWebInterface")
	static TMap<AActor*, FString> FindAllActorsOfClass(UClass* Class);

	/**
	 * Shortcut function to spawn an actor of a class
	 */
	UFUNCTION(BlueprintCallable, Category = "RemoteControlWebInterface")
	static AActor* SpawnActor(UClass* Class);

	/**
	 * Gets all properties values (as a json) of all actors of type Class
	 */
	UFUNCTION(BlueprintCallable, Category = "RemoteControlWebInterface")
	static TMap<AActor*, FString> GetValuesOfActorsByClass(UClass* Class);

private:
	static class URemoteControlPreset* GetPreset(const FString& PresetId);

	static FString GetActorNameOrLabel(const AActor* Actor);
};
