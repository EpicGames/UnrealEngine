// Copyright Epic Games, Inc. All Rights Reserved.

#include "RCWebInterfaceProcess.h"
#include "RCWebInterfacePrivate.h"
#include "RemoteControlSettings.h"
#include "Interfaces/IPluginManager.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/CommandLine.h"
#include "Modules/ModuleManager.h"
#include "IWebRemoteControlModule.h"
#include "IWebSocketNetworkingModule.h"
#include "INetworkingWebSocket.h"
#include "SocketSubsystem.h"


#define LOCTEXT_NAMESPACE "RemoteControlWebInterface"


FRemoteControlWebInterfaceProcess::FRemoteControlWebInterfaceProcess()
	: Status(EStatus::Stopped)
{
}

FRemoteControlWebInterfaceProcess::~FRemoteControlWebInterfaceProcess()
{
}

void FRemoteControlWebInterfaceProcess::Start()
{
	Shutdown();

	Status = EStatus::Launching;

	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	FString WebApp = IPluginManager::Get().FindPlugin("RemoteControlWebInterface")->GetBaseDir() / TEXT("WebApp");
	Root = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*WebApp);

	if (!PlatformFile.DirectoryExists(*Root))
	{
		UE_LOGF(LogRemoteControlWebInterface, Warning, "WebApp folder does not exists (%ls)", *Root);
		return;
	}

	FAsyncTaskNotificationConfig NotificationConfig;
	NotificationConfig.bKeepOpenOnFailure = true;
	NotificationConfig.TitleText = LOCTEXT("RemoteControlWebInterface_Launch", "Launching Remote Control Web Interface");
#if !NO_LOGGING	
	NotificationConfig.LogCategory = &LogRemoteControlWebInterface;
#endif
	NotificationConfig.bIsHeadless = FParse::Param(FCommandLine::Get(), TEXT("RemoteControlIsHeadless"));
	TaskNotification = MakeUnique<FAsyncTaskNotification>(NotificationConfig);

	Thread = FRunnableThread::Create(this, TEXT("FRemoteControlWebInterfaceProcess"), 8 * 1024, TPri_BelowNormal);
}

void FRemoteControlWebInterfaceProcess::Shutdown()
{
	Status = EStatus::Stopped;

	SetExternalLoggerEnabled(false);

	if (Process.IsValid())
	{
		FPlatformProcess::TerminateProc(Process, true);
		Process.Reset();
	}

	if (Thread)
	{
		Thread->Kill(true);
		delete Thread;
		Thread = nullptr;
	}
}

FRemoteControlWebInterfaceProcess::EStatus FRemoteControlWebInterfaceProcess::GetStatus() const
{
	return Status.load();
}

void FRemoteControlWebInterfaceProcess::SetExternalLoggerEnabled(bool bEnableExternalLog)
{
	TSharedPtr<INetworkingWebSocket> WebSocketLoggerConnection;
	if (bEnableExternalLog)
	{
		const URemoteControlSettings* RCSettings = GetDefault<URemoteControlSettings>();

		TSharedRef<FInternetAddr> Address = ISocketSubsystem::Get()->CreateInternetAddr();

		bool bIsValidIp = false;
		Address->SetIp(TEXT("127.0.0.1"), bIsValidIp);
		Address->SetPort(RCSettings->RemoteControlWebInterfacePort + 2);
		WebSocketLoggerConnection = FModuleManager::LoadModuleChecked<IWebSocketNetworkingModule>(TEXT("WebSocketNetworking")).CreateConnection(*Address);
	}


	IWebRemoteControlModule* WebRemoteControlModule = FModuleManager::GetModulePtr<IWebRemoteControlModule>("WebRemoteControl");
	if (WebRemoteControlModule)
	{
		WebRemoteControlModule->SetExternalRemoteWebSocketLoggerConnection(WebSocketLoggerConnection);
	}
}

uint32 FRemoteControlWebInterfaceProcess::Run()
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

#if PLATFORM_WINDOWS
	FString StartScript = Root / TEXT("Start.bat");
#else
	FString StartScript = Root / TEXT("Start.sh");
#endif

	FText ErrorTitle = LOCTEXT("RemoteControlWebInterface_ErrorTitle", "Failed to Launch the Remote Control Web Interface");

	if (!PlatformFile.FileExists(*StartScript))
	{
		TaskNotification->SetComplete(
			ErrorTitle,
			LOCTEXT("RemoteControlWebInterface_FilesMissing", "Missing files in WebInterface folder"),
			false
		);
		Status = EStatus::Error;
		return 0;
	}

	const URemoteControlSettings* RCSettings = GetDefault<URemoteControlSettings>();

	FString Args = FString::Printf(TEXT("--port %d --uews %d --uehttp %d --monitor --hostname %s "),
									RCSettings->RemoteControlWebInterfacePort,
									RCSettings->RemoteControlWebSocketServerPort,
									RCSettings->RemoteControlHttpServerPort,
									*RCSettings->RemoteControlWebInterfaceBindAddress);

	if (RCSettings->bForceWebAppBuildAtStartup)
	{
		Args.Append(TEXT("--build "));
	}

	if (RCSettings->bWebAppLogRequestDuration)
	{
		Args.Append(TEXT("--log "));
	}

	void* ReadPipe = nullptr;
	void* WritePipe = nullptr;

	if (!FPlatformProcess::CreatePipe(ReadPipe, WritePipe))
	{
		TaskNotification->SetComplete(
			ErrorTitle,
			LOCTEXT("RemoteControlWebInterface_PipeFailed", "Failed to create Pipes for the RemoteControlWebInterface process"),
			false
		);
		Status = EStatus::Error;
		return 0;
	}

	check(ReadPipe);
	check(WritePipe);

#if PLATFORM_MAC
	Args = FString::Printf(TEXT("-l \"%s\" %s"), *StartScript, *Args);
	StartScript = TEXT("/bin/sh");
#endif // PLATFORM_MAC

	Process = FPlatformProcess::CreateProc(
		*StartScript,	/* Path to start script */
		*Args,			/* Arguments with port numbers */
		false,			/* bLaunchDetached */
		true,			/* bLaunchHidden */
		true,			/* bLaunchReallyHidden */
		nullptr,		/* OutProcessID */
		0,				/* PriorityModifier */
		*Root,			/* OptionalWorkingDirectory */
		WritePipe		/* PipeWriteChild */
	);

	if (!Process.IsValid())
	{
		FPlatformProcess::ClosePipe(ReadPipe, WritePipe);
		TaskNotification->SetComplete(
			ErrorTitle,
			LOCTEXT("RemoteControlWebInterface_LaunchFailed", "Failed to start RemoteControlWebInterface process"),
			false
		);

		Status = EStatus::Error;
		return 0;
	}

	bool bLoadDone = false;
	auto LogReadPipe = [&](void* InReadPipe) {
		FString ProcessOutput = FPlatformProcess::ReadPipe(InReadPipe);

		if (ProcessOutput.Len() > 0)
		{
			TArray<FString> Lines;
			ProcessOutput.ParseIntoArray(Lines, TEXT("\n"), false);

			for (const FString& Line : Lines)
			{
				if (Line.Len() == 0)
				{
					continue;
				}

				if (!bLoadDone)
				{
					if (Line.StartsWith(TEXT("ERROR: ")))
					{
						bLoadDone = true;
						const FString ErrorMessage = Line.Mid(7);
						TaskNotification->SetComplete(ErrorTitle, FText::FromString(ErrorMessage), false);
						Status = EStatus::Error;
					}
					else if (Line.StartsWith(TEXT("DONE: ")))
					{
						bLoadDone = true;
						const FString CompleteMessage = Line.Mid(6);
						TaskNotification->SetComplete(
							LOCTEXT("RemoteControlWebInterface_SuccessTitle", "Remote Control Web Interface is running"),
							FText::FromString(CompleteMessage),
							true);
						Status = EStatus::Running;


						if (RCSettings->bWebAppLogRequestDuration)
						{
							SetExternalLoggerEnabled(true);
						}
					}
					else
					{
						TaskNotification->SetProgressText(FText::FromString(Line));
					}
				}
				else
				{
					UE_LOGF(LogRemoteControlWebInterface, Log, "%ls", *Line);
				}
			}
		}

		return ProcessOutput.Len();
	};

	UE_LOGF(LogRemoteControlWebInterface, Log, "WebApp started, initial launch will take longer as it will be building the WebApp");

	while (Status != EStatus::Stopped && FPlatformProcess::IsProcRunning(Process))
	{
		const int32 BytesRead = LogReadPipe(ReadPipe);

		if (!BytesRead)
		{
			FPlatformProcess::Sleep(0.2f);
		}
	}

	// One last ReadPipe log for anything that the while loop didn't catch.
	LogReadPipe(ReadPipe);

	if (!bLoadDone)
	{
		TaskNotification->SetComplete(
			ErrorTitle,
			LOCTEXT("RemoteControlWebInterface_AppExited", "WebApp exited"),
			false
		);
	}
	else
	{
		UE_LOGF(LogRemoteControlWebInterface, Log, "WebApp exited");
	}

	Status = EStatus::Stopped;

	FPlatformProcess::ClosePipe(ReadPipe, WritePipe);
	Process.Reset();

	return 0;
}

#undef LOCTEXT_NAMESPACE
