// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Templates/PimplPtr.h"

class FRemoteControlWebInterfaceProcess;

class FRCWebInterfaceCustomizations;

class FRemoteControlWebInterfaceModule 
	: public IModuleInterface
	, public FSelfRegisteringExec
{
public:
	//~ IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	//~ FSelfRegisteringExec interface
	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override;


	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static FRemoteControlWebInterfaceModule& Get()
	{
		static const FName ModuleName = TEXT("RemoteControlWebInterface");
		return FModuleManager::LoadModuleChecked<FRemoteControlWebInterfaceModule>(ModuleName);
	}

private:
	/** Handle Web Interface settings modifications. */
	void OnSettingsModified(UObject* Settings, struct FPropertyChangedEvent& PropertyChangedEvent);

private:
	/** The actual process that runs the middleman server. */
	TSharedPtr<FRemoteControlWebInterfaceProcess> WebApp;

	/** WebSocketServer Start Delegate */
	FDelegateHandle WebSocketServerStartedDelegate;

	/** Customizations/Additions to add to the RC Panel. */
	TPimplPtr<FRCWebInterfaceCustomizations> Customizations;

	/** Prevents the WebApp from starting. Can be set from command line with -RCWebInterfaceDisable */
	bool bRCWebInterfaceDisable;
};
