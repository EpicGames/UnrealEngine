// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Runnable.h"
#include "HAL/RunnableThread.h"
#include "Misc/AsyncTaskNotification.h"
#include <atomic>

class FRemoteControlWebInterfaceProcess : private FRunnable
{
public:
	enum class EStatus : uint8
	{
		Stopped,
		Launching,
		Running,
		Error
	};

public:
	FRemoteControlWebInterfaceProcess();
	
	virtual ~FRemoteControlWebInterfaceProcess();

	/** Starts the webapp external process, first launch will take some time as it needs to compile the webapp */
	void Start();

	/** Shutdown the webapp external process */
	void Shutdown();

	/** Get the webapp's status. */
	EStatus GetStatus() const;

	/** Enable / disable external logger */
	void SetExternalLoggerEnabled(bool bEnableExternalLog);

protected:
	virtual uint32 Run() override;

private:
	/** Plugin root folder */
	FString Root;

	/** WebApp Node.js process handle */
	FProcHandle Process;

	/** Thread for WebApp node.js process (read stdout and forward to log) */
	FRunnableThread* Thread = nullptr;

	/** Progress notification */
	TUniquePtr<FAsyncTaskNotification> TaskNotification;

	/** The current status of the web app. */
	std::atomic<EStatus> Status;
};