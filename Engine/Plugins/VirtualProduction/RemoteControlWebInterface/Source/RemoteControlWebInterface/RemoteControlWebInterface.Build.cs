// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class RemoteControlWebInterface : ModuleRules
{
	public RemoteControlWebInterface(ReadOnlyTargetRules Target) : base(Target)
	{
		ShortName = "RCWebIntf";

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"Projects",
				"RemoteControl",
				"Sockets",
				"WebRemoteControl",
				"WebSocketNetworking",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] 
			{
				"RemoteControlCommon",
				"Serialization",
			}
		);


		if (Target.Type == TargetType.Editor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[] 
				{
					"DeveloperSettings",
					"EditorStyle",
					"EditorWidgets",
					"PropertyEditor",
					"RemoteControl",
					"RemoteControlLogic",
					"RemoteControlUI",
					"Settings",
					"UnrealEd",
					"ToolWidgets",
				}
			);
		}
	}
}
